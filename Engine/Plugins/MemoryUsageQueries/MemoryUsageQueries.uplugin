{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MemoryUsageQueries",
	"Description": "Memory Usage Queries, original contribution from The Coalition (Microsoft) https://thecoalitionstudio.com",
	"Category": "Engine",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Plugins": [
	],
	"Modules": [
		{
			"Name": "MemoryUsageQueries",
			"Type": "Runtime",
			"LoadingPhase": "EarliestPossible"
		}
	]
}