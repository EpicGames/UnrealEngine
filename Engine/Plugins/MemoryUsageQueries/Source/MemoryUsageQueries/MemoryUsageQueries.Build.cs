// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MemoryUsageQueries : ModuleRules
	{
		public MemoryUsageQueries(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicIncludePaths.AddRange(
				new string[] {
					"$(ModuleDir)/Public",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"Engine",
					"EngineSettings",
					"PakFile"
				}
			);
		}
	}
}