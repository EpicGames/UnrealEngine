// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryUsageInfoProvider.h"

uint64 IMemoryUsageInfoProvider::GetAssetMemoryUsage(FName Asset, FOutputDevice* ErrorOutput) const
{
	ErrorOutput->Logf(TEXT("MemoryUsageInfoProvider Error: No provider is available."));
	return 0U;
}
uint64 IMemoryUsageInfoProvider::GetAssetsMemoryUsage(const TSet<FName>& Assets, FOutputDevice* ErrorOutput) const
{
	ErrorOutput->Logf(TEXT("MemoryUsageInfoProvider Error: No provider is available."));
	return 0U;
}
uint64 IMemoryUsageInfoProvider::GetAssetsMemoryUsageWithSize(const TSet<FName>& Assets, TMap<FName, uint64>& OutSizes, FOutputDevice* ErrorOutput) const
{
	ErrorOutput->Logf(TEXT("MemoryUsageInfoProvider Error: No provider is available."));
	return 0U;
}
void IMemoryUsageInfoProvider::GetAllAssetsWithSize(TMap<FName, uint64>& OutAssets, FOutputDevice* ErrorOutput) const
{
	ErrorOutput->Logf(TEXT("MemoryUsageInfoProvider Error: No provider is available."));
}
