// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryUsageInfoProviderLLM.h"
#include "MemoryUsageQueries.h"
#include "MemoryUsageQueriesPrivate.h"
#include "Templates/Greater.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

bool FMemoryUsageInfoProviderLLM::IsProviderAvailable() const
{
	return FLowLevelMemTracker::IsEnabled();
}

uint64 FMemoryUsageInfoProviderLLM::GetAssetMemoryUsage(FName Asset) const
{
	if (FLowLevelMemTracker::IsEnabled())
	{
		return FLowLevelMemTracker::Get().GetTagAmountForTracker(ELLMTracker::Default, Asset, ELLMTagSet::Assets, UE::LLM::ESizeParams::Default);
	}

	return 0U;
}

uint64 FMemoryUsageInfoProviderLLM::GetAssetsMemoryUsage(const TSet<FName>& Assets) const
{
	if (FLowLevelMemTracker::IsEnabled())
	{
		uint64 TotalSize = 0;
		for (const FName& Asset : Assets)
		{
			TotalSize += FLowLevelMemTracker::Get().GetTagAmountForTracker(ELLMTracker::Default, Asset, ELLMTagSet::Assets, UE::LLM::ESizeParams::Default);
		}
		return TotalSize;
	}

	return 0U;
}

uint64 FMemoryUsageInfoProviderLLM::GetAssetsMemoryUsageWithSize(const TSet<FName>& Assets, TMap<FName, uint64>& OutSizes) const
{
	if (FLowLevelMemTracker::IsEnabled())
	{
		uint64 TotalSize = 0;
		for (const FName& Asset : Assets)
		{
			int64 AssetSize = FLowLevelMemTracker::Get().GetTagAmountForTracker(ELLMTracker::Default, Asset, ELLMTagSet::Assets, UE::LLM::ESizeParams::Default);
			OutSizes.Add(Asset, AssetSize);
			TotalSize += AssetSize;
		}
		return TotalSize;
	}

	return 0U;
}

void FMemoryUsageInfoProviderLLM::GetAllAssetsWithSize(TMap<FName, uint64>& OutAssets) const
{
	OutAssets.Reset();
	if (FLowLevelMemTracker::IsEnabled())
	{
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(OutAssets, ELLMTracker::Default, ELLMTagSet::Assets);
	}
}

void FMemoryUsageInfoProviderLLM::GetFilteredTagsWithSize(TMap<FName, uint64>& OutTags, ELLMTracker Tracker, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters) const
{
	OutTags.Reset();
	if (FLowLevelMemTracker::IsEnabled())
	{
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmountFiltered(OutTags, Tracker, TagSet, Filters);
		return;
	}
}

namespace MemoryUsageQueries
{

namespace Internal
{
	FMemoryUsageInfoProviderLLM GMemoryUsageInfoProviderLLM;
}

IMemoryUsageInfoProvider* GetCurrentMemoryUsageInfoProvider()
{
	return &Internal::GMemoryUsageInfoProviderLLM;
}

bool GetFilteredPackagesWithSize(TMap<FName, uint64>& OutPackagesWithSize, FName GroupName /* = NAME_None */, FString AssetSubstring /* = FString() */, FName ClassName /* = NAME_None */, FOutputDevice* ErrorOutput /*= GLog*/)
{
	TArray<FLLMTagSetAllocationFilter> Filters;

	FName LongName = NAME_None;
	if (!AssetSubstring.IsEmpty())
	{
		Internal::GetLongName(AssetSubstring, LongName, ErrorOutput);
	}

	if (LongName != NAME_None)
	{
		Filters.Add({ LongName, ELLMTagSet::Assets });
	}

	if (GroupName != NAME_None)
	{
		Filters.Add({GroupName, ELLMTagSet::None});
	}

	if (ClassName != NAME_None)
	{
		Filters.Add({ClassName, ELLMTagSet::AssetClasses});
	}

	Internal::GMemoryUsageInfoProviderLLM.GetFilteredTagsWithSize(OutPackagesWithSize, ELLMTracker::Default, ELLMTagSet::Assets, Filters);

	if (AssetSubstring.Len() > 0)
	{
		Internal::RemoveFilteredPackages(OutPackagesWithSize, AssetSubstring);
	}

	OutPackagesWithSize.ValueSort(TGreater<uint64>());

	return true;
}

bool GetFilteredPackagesCategorizedWithSize(TMap<FName, uint64>& OutPackagesWithSize, FName GroupName /*= NAME_None*/, FString AssetSubstring /*= FString()*/, FName ClassName /*= NAME_None*/, FName CategoryName /*= NAME_None*/, FOutputDevice* ErrorOutput /*= GLog*/)
{
	TArray<FLLMTagSetAllocationFilter> Filters;

	FName LongName = NAME_None;
	if (!AssetSubstring.IsEmpty())
	{
		Internal::GetLongName(AssetSubstring, LongName, ErrorOutput);
	}

	if (LongName != NAME_None)
	{
		Filters.Add({ LongName, ELLMTagSet::Assets });
	}

	if (GroupName != NAME_None)
	{
		Filters.Add({ GroupName, ELLMTagSet::None });
	}

	if (ClassName != NAME_None)
	{
		Filters.Add({ ClassName, ELLMTagSet::AssetClasses });
	}

	ELLMTagSet TagSetCategory = ELLMTagSet::None;
	if (CategoryName == FName("Assets"))
	{
		TagSetCategory = ELLMTagSet::Assets;
	}
	else  if (CategoryName == FName("AssetClasses"))
	{
		TagSetCategory = ELLMTagSet::AssetClasses;
	}

	Internal::GMemoryUsageInfoProviderLLM.GetFilteredTagsWithSize(OutPackagesWithSize, ELLMTracker::Default, TagSetCategory, Filters);

	if (AssetSubstring.Len() > 0)
	{
		Internal::RemoveFilteredPackages(OutPackagesWithSize, AssetSubstring);
	}

	OutPackagesWithSize.ValueSort(TGreater<uint64>());

	return true;
}

bool GetFilteredClassesWithSize(TMap<FName, uint64>& OutClassesWithSize, FName GroupName /* = NAME_None */, FString AssetName /* = FString() */, FOutputDevice* ErrorOutput /* = GLog */)
{
	TArray<FLLMTagSetAllocationFilter> Filters;

	FName LongName = NAME_None;
	if (!AssetName.IsEmpty() && !Internal::GetLongName(AssetName, LongName, ErrorOutput))
	{
		return false;
	}

	if (LongName != NAME_None)
	{
		Filters.Add({LongName, ELLMTagSet::Assets});
	}

	if (GroupName != NAME_None)
	{
		Filters.Add({GroupName, ELLMTagSet::None});
	}

	Internal::GMemoryUsageInfoProviderLLM.GetFilteredTagsWithSize(OutClassesWithSize, ELLMTracker::Default, ELLMTagSet::AssetClasses, Filters);
	
	OutClassesWithSize.ValueSort(TGreater<uint64>());

	return true;
}

bool GetFilteredGroupsWithSize(TMap<FName, uint64>& OutGroupsWithSize, FString AssetName /* = FString() */, FName ClassName /* = NAME_None */, FOutputDevice* ErrorOutput /* = GLog */)
{
	TArray<FLLMTagSetAllocationFilter> Filters;

	FName LongName = NAME_None;
	if (!AssetName.IsEmpty() && !Internal::GetLongName(AssetName, LongName, ErrorOutput))
	{
		return false;
	}

	if (LongName != NAME_None)
	{
		Filters.Add({LongName, ELLMTagSet::Assets});
	}

	if (ClassName != NAME_None)
	{
		Filters.Add({ClassName, ELLMTagSet::AssetClasses});
	}

	Internal::GMemoryUsageInfoProviderLLM.GetFilteredTagsWithSize(OutGroupsWithSize, ELLMTracker::Default, ELLMTagSet::None, Filters);

	OutGroupsWithSize.ValueSort(TGreater<uint64>());

	return true;
}

} // namespace MemoryUsageQueries
#else // ENABLE_LOW_LEVEL_MEM_TRACKER
namespace MemoryUsageQueries
{
// For now only LLM backend is supported
IMemoryUsageInfoProvider* GetCurrentMemoryUsageInfoProvider()
{
	return nullptr;
}
} // namespace MemoryUsageQueries
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER
