// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "MemoryUsageInfoProvider.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/OutputDeviceRedirector.h"

#define UE_API MEMORYUSAGEQUERIES_API

#if ENABLE_LOW_LEVEL_MEM_TRACKER

class FMemoryUsageInfoProviderLLM : public IMemoryUsageInfoProvider
{
public:
	UE_API virtual bool IsProviderAvailable() const override;
	UE_API virtual uint64 GetAssetMemoryUsage(FName Asset) const override;
	UE_API virtual uint64 GetAssetsMemoryUsage(const TSet<FName>& Assets) const override;
	UE_API virtual void GetAllAssetsWithSize(TMap<FName, uint64>& OutAssets) const override;
	UE_API virtual uint64 GetAssetsMemoryUsageWithSize(const TSet<FName>& Assets, TMap<FName, uint64>& OutSizes) const override;
	UE_API virtual void GetFilteredTagsWithSize(TMap<FName, uint64>& OutTags, ELLMTracker Tracker, ELLMTagSet TagSet, TArray<FLLMTagSetAllocationFilter>& Filters) const;
};

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

#undef UE_API
