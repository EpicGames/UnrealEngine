// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryUsageQueries.h"

#include "ConsoleSettings.h"
#include "IPlatformFilePak.h"
#include "MemoryUsageQueriesConfig.h"
#include "MemoryUsageQueriesPrivate.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/FileManager.h"
#include "Serialization/PackageStore.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/PackageName.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "Templates/Greater.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectIterator.h"

namespace MemoryUsageQueries
{

void GetMemoryUsage(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FName& PackageName, uint64& OutExclusiveSize, uint64& OutInclusiveSize)
{
	TSet<FName> Dependencies;
	Internal::GetTransitiveDependencies(PackageName, Dependencies);
	Dependencies.Add(PackageName);
	OutExclusiveSize = MemoryUsageInfoProvider->GetAssetMemoryUsage(PackageName);
	OutInclusiveSize = MemoryUsageInfoProvider->GetAssetsMemoryUsage(Dependencies);
}
bool GetMemoryUsage(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FString& PackageName, uint64& OutExclusiveSize, uint64& OutInclusiveSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	FName LongPackageName;
	if (!Internal::GetLongName(PackageName, LongPackageName, ErrorOutput))
	{
		return false;
	}

	GetMemoryUsage(MemoryUsageInfoProvider, LongPackageName, OutExclusiveSize, OutInclusiveSize);
	return true;
}

void GetMemoryUsageCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutTotalSize)
{
	TSet<FName> Dependencies;
	Internal::GetDependenciesCombined(PackageNames, Dependencies);
	OutTotalSize =  MemoryUsageInfoProvider->GetAssetsMemoryUsage(Dependencies);
}
bool GetMemoryUsageCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutTotalSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetMemoryUsageCombined(MemoryUsageInfoProvider, LongPackageNames.Array(), OutTotalSize);
	return true;
}

void GetMemoryUsageShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutTotalSize)
{
	TSet<FName> Dependencies;
	Internal::GetDependenciesShared(PackageNames, Dependencies);
	OutTotalSize = MemoryUsageInfoProvider->GetAssetsMemoryUsage(Dependencies);
}
bool GetMemoryUsageShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutTotalSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetMemoryUsageShared(MemoryUsageInfoProvider, LongPackageNames.Array(), OutTotalSize);
	return true;
}

void GetMemoryUsageUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutUniqueSize)
{
	TSet<FName> RemovablePackages;
	Internal::GetRemovablePackages(PackageNames, RemovablePackages);
	OutUniqueSize = MemoryUsageInfoProvider->GetAssetsMemoryUsage(RemovablePackages);
}
bool GetMemoryUsageUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutUniqueSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetMemoryUsageUnique(MemoryUsageInfoProvider, LongPackageNames.Array(), OutUniqueSize);
	return true;
}

void GetMemoryUsageCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutCommonSize)
{
	TSet<FName> UnremovablePackages;
	Internal::GetUnremovablePackages(PackageNames, UnremovablePackages);
	OutCommonSize = MemoryUsageInfoProvider->GetAssetsMemoryUsage(UnremovablePackages);
}
bool GetMemoryUsageCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutCommonSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetMemoryUsageCommon(MemoryUsageInfoProvider, LongPackageNames.Array(), OutCommonSize);
	return true;
}

void GatherDependenciesForPackages(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames,
	TMap<FName, uint64>& OutInternalDeps, TMap<FName, uint64>& OutExternalDeps, MemoryUsageQueries::EDependencyType DependencyType)
{
	const bool bFindRemovable = (DependencyType == MemoryUsageQueries::EDependencyType::EDep_All || DependencyType == MemoryUsageQueries::EDependencyType::EDep_Removable);
	const bool bFindNonRemovable = (DependencyType == MemoryUsageQueries::EDependencyType::EDep_All || DependencyType == MemoryUsageQueries::EDependencyType::EDep_NonRemovable);

	if (bFindRemovable)
	{
		TSet<FName> RemovablePackages;
		Internal::GetRemovablePackages(PackageNames, RemovablePackages);
		MemoryUsageInfoProvider->GetAssetsMemoryUsageWithSize(RemovablePackages, OutInternalDeps);
	}
	if (bFindNonRemovable)
	{
		TSet<FName> UnremovablePackages;
		Internal::GetUnremovablePackages(PackageNames, UnremovablePackages);
		MemoryUsageInfoProvider->GetAssetsMemoryUsageWithSize(UnremovablePackages, OutExternalDeps);
	}

}
bool GatherDependenciesForPackages(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames,
	TMap<FName, uint64>& OutInternalDeps, TMap<FName, uint64>& OutExternalDeps, MemoryUsageQueries::EDependencyType DependencyType, FOutputDevice* ErrorOutput)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}
	GatherDependenciesForPackages(MemoryUsageInfoProvider, LongPackageNames.Array(), OutInternalDeps, OutExternalDeps, DependencyType);
	return true;
}

void GetDependenciesWithSize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FName& PackageName, TMap<FName, uint64>& OutDependenciesWithSize)
{
	TSet<FName> Dependencies;
	Internal::GetTransitiveDependencies(PackageName, Dependencies);
	Dependencies.Add(PackageName);
	OutDependenciesWithSize.Empty();
	Internal::SortPackagesBySize(MemoryUsageInfoProvider, Dependencies, OutDependenciesWithSize);
}
bool GetDependenciesWithSize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FString& PackageName, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	FName LongPackageName;
	if (!Internal::GetLongName(PackageName, LongPackageName, ErrorOutput))
	{
		return false;
	}

	GetDependenciesWithSize(MemoryUsageInfoProvider, LongPackageName, OutDependenciesWithSize);
	return true;
}

void GetDependenciesWithSizeCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize)
{
	TSet<FName> Dependencies;
	Internal::GetDependenciesCombined(PackageNames, Dependencies);
	OutDependenciesWithSize.Empty();
	Internal::SortPackagesBySize(MemoryUsageInfoProvider, Dependencies, OutDependenciesWithSize);
}
bool GetDependenciesWithSizeCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetDependenciesWithSizeCombined(MemoryUsageInfoProvider, LongPackageNames.Array(), OutDependenciesWithSize);
	return true;
}
void GetDependenciesWithSizeShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize)
{
	TSet<FName> Dependencies;
	Internal::GetDependenciesShared(PackageNames, Dependencies);
	OutDependenciesWithSize.Empty();
	Internal::SortPackagesBySize(MemoryUsageInfoProvider, Dependencies, OutDependenciesWithSize);
}
bool GetDependenciesWithSizeShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetDependenciesWithSizeShared(MemoryUsageInfoProvider, LongPackageNames.Array(), OutDependenciesWithSize);
	return true;
}

void GetDependenciesWithSizeUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize)
{
	TSet<FName> RemovablePackages;
	Internal::GetRemovablePackages(PackageNames, RemovablePackages);
	OutDependenciesWithSize.Empty();
	Internal::SortPackagesBySize(MemoryUsageInfoProvider, RemovablePackages, OutDependenciesWithSize);
}
bool GetDependenciesWithSizeUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetDependenciesWithSizeUnique(MemoryUsageInfoProvider, LongPackageNames.Array(), OutDependenciesWithSize);
	return true;
}

void GetDependenciesWithSizeCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize)
{
	TSet<FName> UnremovablePackages;
	Internal::GetUnremovablePackages(PackageNames, UnremovablePackages);
	OutDependenciesWithSize.Empty();
	Internal::SortPackagesBySize(MemoryUsageInfoProvider, UnremovablePackages, OutDependenciesWithSize);
}
bool GetDependenciesWithSizeCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (!MemoryUsageInfoProvider->IsProviderAvailable())
	{
		if (ErrorOutput)
		{
			ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		}
		return false;
	}
	TSet<FName> LongPackageNames;
	if (!Internal::GetLongNames(PackageNames, LongPackageNames, ErrorOutput))
	{
		return false;
	}

	GetDependenciesWithSizeCommon(MemoryUsageInfoProvider, LongPackageNames.Array(), OutDependenciesWithSize);
	return true;
}

namespace Internal
{
FMemoryUsageReferenceProcessor::FMemoryUsageReferenceProcessor()
{
	int32 Num = GUObjectArray.GetObjectArrayNum();

	Excluded.Init(false, Num);
	ReachableFull.Init(false, Num);
	ReachableExcluded.Init(false, Num);
}

void FMemoryUsageReferenceProcessor::Init(const TArray<FName>& PackageNames)
{
	for (FRawObjectIterator It(true); It; ++It)
	{
		FUObjectItem* ObjectItem = *It;
		UObject* Object = (UObject*)ObjectItem->GetObject();

		if (ObjectItem->IsUnreachable())
		{
			continue;
		}

		if (ObjectItem->IsRootSet())
		{
			RootSetPackages.Add(Object);
		}

		if (UClass* Class = dynamic_cast<UClass*>(Object))
		{
			if (!Class->HasAnyClassFlags(CLASS_TokenStreamAssembled))
			{
				Class->AssembleReferenceTokenStream();
				check(Class->HasAnyClassFlags(CLASS_TokenStreamAssembled));
			}
		}
	}

	if (FPlatformProperties::RequiresCookedData() && FGCObject::GGCObjectReferencer && GUObjectArray.IsDisregardForGC(FGCObject::GGCObjectReferencer))
	{
		RootSetPackages.Add(FGCObject::GGCObjectReferencer);
	}

	for (const auto& PackageName : PackageNames)
	{
		UPackage* Package = Cast<UPackage>(StaticFindObjectFast(UPackage::StaticClass(), nullptr, PackageName, EFindObjectFlags::ExactClass));
		if (Package)
		{
			auto ExcludeObject = [&](UObject* ObjectToExclude) {
				if (ObjectToExclude && GUObjectArray.ObjectToIndex(ObjectToExclude) < Excluded.Num())
				{
					Excluded[GUObjectArray.ObjectToIndex(ObjectToExclude)] = true;
				}
			};

			auto ExcludeObjectOfClass = [&](UObject* ObjectOfClass) {
				if (ObjectOfClass)
				{
					ForEachObjectWithOuter(ObjectOfClass, ExcludeObject);
				}
				ExcludeObject(ObjectOfClass);
			};

			auto ExcludeObjectInPackage = [&](UObject* ObjectInPackage) {
				if (ObjectInPackage->IsA<UClass>())
				{
					UClass* Class = static_cast<UClass*>(ObjectInPackage);
					ForEachObjectOfClass(Class, ExcludeObjectOfClass);
				}
				ExcludeObject(ObjectInPackage);
			};

			ForEachObjectWithOuter(Package, ExcludeObjectInPackage);
			ExcludeObject(Package);
		}
	}
}

TArray<UObject*>& FMemoryUsageReferenceProcessor::GetRootSet()
{
	return RootSetPackages;
}

void FMemoryUsageReferenceProcessor::HandleTokenStreamObjectReference(UE::GC::FWorkerContext& Context, const UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId TokenIndex, const EGCTokenType TokenType, bool bAllowReferenceElimination)
{
	FPermanentObjectPoolExtents PermanentPool;
	if (Object == nullptr || GUObjectArray.ObjectToIndex(Object) >= ReachableFull.Num() || PermanentPool.Contains(Object) || GUObjectArray.IsDisregardForGC(Object))
	{
		return;
	}

	int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);

	if (Mode == EMode::Full)
	{
		if (!ReachableFull[ObjectIndex])
		{
			ReachableFull[ObjectIndex] = true;
			Context.ObjectsToSerialize.Add<Options>(Object);
		}
	}
	else if (Mode == EMode::Excluding)
	{
		if (!ReachableExcluded[ObjectIndex] && !Excluded[ObjectIndex])
		{
			ReachableExcluded[ObjectIndex] = true;
			Context.ObjectsToSerialize.Add<Options>(Object);
		}
	}
}

void FMemoryUsageReferenceProcessor::GetUnreachablePackages(TSet<FName>& OutUnreachablePackages)
{
	for (int i = 0; i < ReachableFull.Num(); i++)
	{
		if (ReachableFull[i] && !ReachableExcluded[i])
		{
			UObject* Obj = static_cast<UObject*>(GUObjectArray.IndexToObjectUnsafeForGC(i)->GetObject());
			if (Obj && Obj->IsA<UPackage>())
			{
				OutUnreachablePackages.Add(Obj->GetFName());
			}
		}
	}
}


static FAutoConsoleVariable CVarMemQueryUsePackageStore(
	TEXT("MemQuery.UsePackageStore"), true,
	TEXT("True - use PackageStore, false - use AssetRegistry."), ECVF_Default);

static FAssetRegistryModule& GetAssetRegistryModule()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	if (IsInGameThread())
	{
		AssetRegistryModule.Get().WaitForCompletion();
	}

	return AssetRegistryModule;
}

class FPackageStoreLazyDatabase final
{
	FPackageStoreLazyDatabase()
	{
		FPackageName::OnContentPathMounted().AddRaw(this, &FPackageStoreLazyDatabase::OnContentPathMounted);
		FPackageName::OnContentPathDismounted().AddRaw(this, &FPackageStoreLazyDatabase::OnContentPathDismounted);
	}

	~FPackageStoreLazyDatabase()
	{
		FPackageName::OnContentPathMounted().RemoveAll(this);
		FPackageName::OnContentPathDismounted().RemoveAll(this);
	}

	void ResetDatabase()
	{
		bIsAssetDatabaseSearched = false;
		bIsDirectoryIndexSearched = false;
		Names.Empty();
		Database.Empty();
	}

	void OnContentPathMounted(const FString&, const FString&) { ResetDatabase(); }
	void OnContentPathDismounted(const FString&, const FString&) { ResetDatabase(); }

	bool BuildDatabaseWhile(TFunctionRef<bool(const FPackageId& PackageId, const FName& PackageName)> Predicate)
	{
		bool bIterationBrokeEarly = false;

		if (!bIsAssetDatabaseSearched)
		{
			const FAssetRegistryModule& AssetRegistryModule = GetAssetRegistryModule();
			AssetRegistryModule.Get().EnumerateAllAssets(
				[&](const FAssetData& Data)
				{
					FPackageId PackageId = FPackageId::FromName(Data.PackageName);
					{
						LLM_SCOPE_BYNAME("MemoryUsageQueries");
						Database.FindOrAdd(PackageId, Data.PackageName);
						Names.Add(Data.PackageName);
					}

					if (!Invoke(Predicate, PackageId, Data.PackageName))
					{
						bIterationBrokeEarly = true;
						return false;
					}

					return true;
				},
				UE::AssetRegistry::EEnumerateAssetsFlags::OnlyOnDiskAssets);

			if (bIterationBrokeEarly)
			{
				return false;
			}

			bIsAssetDatabaseSearched = true;
		}

		if (!bIsDirectoryIndexSearched)
		{
			FPakPlatformFile::ForeachPackageInIostoreWhile(
				[&](FName PackageName)
				{
					FPackageId PackageId = FPackageId::FromName(PackageName);
					{
						LLM_SCOPE_BYNAME("MemoryUsageQueries");
						Database.FindOrAdd(PackageId, PackageName);
						Names.Add(PackageName);
					}

					if (!Invoke(Predicate, PackageId, PackageName))
					{
						bIterationBrokeEarly = true;
						return false;
					}

					return true;
				});

			if (bIterationBrokeEarly)
			{
				return false;
			}

			bIsDirectoryIndexSearched = true;
		}

		return true;
	}

	TMap<FPackageId, FName> Database;
	TArray<FName> Names;
	bool bIsAssetDatabaseSearched = false;
	bool bIsDirectoryIndexSearched = false;

public:
	static FPackageStoreLazyDatabase& Get()
	{
		static FPackageStoreLazyDatabase Instance{};
		return Instance;
	}

	// Blocking Call, builds full database
	void IterateAllPackages(TFunctionRef<void(const FPackageId&)> Visitor)
	{
		BuildDatabaseWhile([](const auto&, const auto&) { return true; });

		for (const TTuple<FPackageId, FName>& Pair : Database)
		{
			Invoke(Visitor, Pair.Key);
		}
	}


	bool GetPackageNameFromId(FPackageId InPackageId, FName& OutFullName)
	{
		if (const FName* Found = Database.Find(InPackageId))
		{
			OutFullName = *Found;
			return true;
		}

		const bool bIterationCompletedWithoutBreak = BuildDatabaseWhile(
			[&](const FPackageId& PackageId, const FName& PackageName)
			{
				if (PackageId == InPackageId)
				{
					OutFullName = PackageName;
					return false;
				}

				return true;
			});

		return !bIterationCompletedWithoutBreak;
	}

	bool GetFirstPackageNameFromPartialName(FStringView InPartialName, FName& OutPackageName)
	{
		auto SearchCriteria = [InPartialName](const FName& PackageName) -> bool
		{
			TCHAR Storage[FName::StringBufferSize];
			PackageName.ToString(Storage);
			return UE::String::FindFirst(Storage, InPartialName, ESearchCase::IgnoreCase) != INDEX_NONE;
		};

		if (const FName* Found = Names.FindByPredicate(SearchCriteria))
		{
			OutPackageName = *Found;
			return true;
		}

		const bool bIterationCompletedWithoutBreak = BuildDatabaseWhile(
			[&](const FPackageId&, const FName& PackageName)
			{
				if (SearchCriteria(PackageName))
				{
					OutPackageName = PackageName;
					return false;
				}

				return true;
			});

		return !bIterationCompletedWithoutBreak;
	}

	bool DoesPackageExists(const FName& PackageName)
	{
		BuildDatabaseWhile([](const auto&, const auto&) { return true; });
		return Names.Contains(PackageName);
	}
};

class FPackageDependenciesLazyDatabase final
{
	FPackageDependenciesLazyDatabase()
	{
		FPackageName::OnContentPathMounted().AddRaw(this, &FPackageDependenciesLazyDatabase::OnContentPathMounted);
		FPackageName::OnContentPathDismounted().AddRaw(this, &FPackageDependenciesLazyDatabase::OnContentPathDismounted);
	}

	~FPackageDependenciesLazyDatabase()
	{
		FPackageName::OnContentPathMounted().RemoveAll(this);
		FPackageName::OnContentPathDismounted().RemoveAll(this);
	}

	void ResetDatabase()
	{
		Dependencies.Empty();
		Referencers.Empty();
		Leafs.Empty();
		Roots.Empty();
	}

	void OnContentPathMounted(const FString&, const FString&) { ResetDatabase(); }
	void OnContentPathDismounted(const FString&, const FString&) { ResetDatabase(); }

	bool InsertPackage(const FPackageId RootPackageId)
	{
		LLM_SCOPE_BYNAME("MemoryUsageQueries");

		TArray<FPackageId, TInlineAllocator<2048>> Stack;

		Stack.Add(RootPackageId);
		bool bAddedSuccessfully = false;

		while (!Stack.IsEmpty())
		{
			const FPackageId PackageId = Stack.Pop(EAllowShrinking::No);

			if (Dependencies.Contains(PackageId) || Leafs.Contains(PackageId))
			{
				bAddedSuccessfully = true;
				continue;
			}

			FPackageStoreEntry PackageEntry;
			const EPackageStoreEntryStatus Status = FPackageStore::Get().GetPackageStoreEntry(PackageId, NAME_None,
				PackageEntry);
			if (Status == EPackageStoreEntryStatus::Ok)
			{
				// add package dependencies
				const TArrayView<const FPackageId> ImportedPackageIds = PackageEntry.ImportedPackageIds;
				for (FPackageId DependentId : ImportedPackageIds)
				{
					Dependencies.FindOrAdd(PackageId).Add(DependentId);
					Referencers.FindOrAdd(DependentId).Add(PackageId);
					bAddedSuccessfully = true;
				}
				Stack.Append(ImportedPackageIds);

#if WITH_EDITOR
				// Add editor optional dependencies
				const TArrayView<const FPackageId> OptionalImportedPackageIds = PackageEntry.OptionalSegmentImportedPackageIds;
				for (FPackageId DependentId : OptionalImportedPackageIds)
				{
					Dependencies.FindOrAdd(PackageId).Add(DependentId);
					Referencers.FindOrAdd(DependentId).Add(PackageId);
					bAddedSuccessfully = true;
				}
				Stack.Append(OptionalImportedPackageIds);
#endif


				// Add leafs
				if (ImportedPackageIds.IsEmpty()
#if WITH_EDITOR
					&& OptionalImportedPackageIds.IsEmpty()
#endif
				)
				{
					Leafs.Add(PackageId);
					bAddedSuccessfully = true;
				}
			}
		}

		return bAddedSuccessfully;
	}

	TMap<FPackageId, TSet<FPackageId>> Dependencies;
	TMap<FPackageId, TSet<FPackageId>> Referencers;
	TSet<FPackageId> Leafs;
	TSet<FPackageId> Roots;

public:
	static FPackageDependenciesLazyDatabase& Get()
	{
		static FPackageDependenciesLazyDatabase Instance{};
		return Instance;
	}

	bool GetDependencies(const FPackageId PackageId, TSet<FPackageId>& OutDependencies)
	{
		if (Leafs.Contains(PackageId))
		{
			return true;
		}

		const TSet<FPackageId>* ChildrenSet = Dependencies.Find(PackageId);
		if (ChildrenSet == nullptr)
		{
			FPackageStoreReadScope _(FPackageStore::Get());
			if (!InsertPackage(PackageId))
			{
				return false;
			}
			ChildrenSet = Dependencies.Find(PackageId);
		}
		if (ChildrenSet == nullptr)
		{
			return false;
		}

		TArray<FPackageId, TInlineAllocator<2048>> Stack(ChildrenSet->Array());
		while (!Stack.IsEmpty())
		{
			FPackageId Child = Stack.Pop(EAllowShrinking::No);

			if (!OutDependencies.Contains(Child))
			{
				OutDependencies.Add(Child);
				if (const TSet<FPackageId>* GrandChildrenSet = Dependencies.Find(Child))
				{
					Stack.Append(GrandChildrenSet->Array());
				}
			}
		}

		return true;
	}

	bool GetReferencers(const FPackageId InPackageId, TSet<FPackageId>& OutReferencers)
	{
		if (Roots.Contains(InPackageId))
		{
			return true;
		}

		TSet<FPackageId>* ParentsSet = Referencers.Find(InPackageId);
		if (ParentsSet == nullptr)
		{
			FPackageStoreReadScope _(FPackageStore::Get());
			// First IterateAllPackages call builds dependencies map
			FPackageStoreLazyDatabase::Get().IterateAllPackages([this](const FPackageId& PackageId) { InsertPackage(PackageId); });
			// Second IterateAllPackages call caches root nodes (only valid after full database is built)
			FPackageStoreLazyDatabase::Get().IterateAllPackages(
				[this](const FPackageId& PackageId)
				{
					if (!Referencers.Contains(PackageId))
					{
						Roots.Add(PackageId);
					}
				});

			ParentsSet = Referencers.Find(InPackageId);
		}

		OutReferencers = MoveTemp(*ParentsSet);

		return true;
	}
};

static bool GetLongNamePackageStore(FStringView InShortPackageName, FName& OutLongPackageName)
{
	return FPackageStoreLazyDatabase::Get().GetFirstPackageNameFromPartialName(InShortPackageName, OutLongPackageName);
}

static bool GetLongNameAssetRegistry(FStringView InShortPackageName, FName& OutLongPackageName)
{
	const FAssetRegistryModule& AssetRegistryModule = GetAssetRegistryModule();

	OutLongPackageName = AssetRegistryModule.Get().GetFirstPackageByName(InShortPackageName);
	if (OutLongPackageName == NAME_None)
	{
		return false;
	}

	return true;
}

bool GetLongName(FStringView ShortPackageName, FName& OutLongPackageName, FOutputDevice* ErrorOutput /* = GLog */)
{
	if (FPackageName::IsValidLongPackageName(ShortPackageName))
	{
		OutLongPackageName = FName(ShortPackageName);
		return true;
	}

	const bool Result =
		CVarMemQueryUsePackageStore->GetBool() && FPackageStore::Get().HasAnyBackendsMounted()
		? GetLongNamePackageStore(ShortPackageName, OutLongPackageName)
		: GetLongNameAssetRegistry(ShortPackageName, OutLongPackageName);

	if (!Result && ErrorOutput)
	{
		ErrorOutput->Logf(ELogVerbosity::Error, TEXT("MemQuery Error: Package not found: %.*s"), ShortPackageName.Len(), ShortPackageName.GetData());
	}
	return Result;
}

bool GetLongNames(const TArray<FString>& PackageNames, TSet<FName>& OutLongPackageNames, FOutputDevice* ErrorOutput /* = GLog */)
{
	for (const auto& Package : PackageNames)
	{
		FName LongName;
		if (!GetLongName(Package, LongName, ErrorOutput))
		{
			return false;
		}
		OutLongPackageNames.Add(LongName);
	}

	return true;
}

void GetDependenciesCombined(const TArray<FName>& PackageNames, TSet<FName>& OutDependencies)
{
	TSet<FName> Dependencies;
	for (const auto& PackageName : PackageNames)
	{
		Dependencies.Empty();
		Internal::GetTransitiveDependencies(PackageName, Dependencies);
		OutDependencies.Append(Dependencies);
	}
	OutDependencies.Append(PackageNames);
}

void GetDependenciesShared(const TArray<FName>& PackageNames, TSet<FName>& OutDependencies)
{
	TSet<FName> Dependencies;

	for (int i = 0; i < PackageNames.Num(); i++)
	{
		Dependencies.Empty();
		Internal::GetTransitiveDependencies(PackageNames[i], Dependencies);
		Dependencies.Add(PackageNames[i]);

		if (i == 0)
		{
			OutDependencies.Append(Dependencies);
			continue;
		}

		OutDependencies = OutDependencies.Intersect(Dependencies);
	}
}

void PerformReachabilityAnalysis(FMemoryUsageReferenceProcessor& ReferenceProcessor)
{
	{
		FGCArrayStruct ArrayStruct;
		ArrayStruct.SetInitialObjectsUnpadded(ReferenceProcessor.GetRootSet());
		ReferenceProcessor.SetMode(FMemoryUsageReferenceProcessor::Full);
		CollectReferences<FMemoryUsageReferenceCollector, FMemoryUsageReferenceProcessor>(ReferenceProcessor, ArrayStruct);
	}

	{
		FGCArrayStruct ArrayStruct;
		ArrayStruct.SetInitialObjectsUnpadded(ReferenceProcessor.GetRootSet());
		ReferenceProcessor.SetMode(FMemoryUsageReferenceProcessor::Excluding);
		CollectReferences<FMemoryUsageReferenceCollector, FMemoryUsageReferenceProcessor>(ReferenceProcessor, ArrayStruct);
	}
}

void GetRemovablePackages(const TArray<FName>& PackagesToUnload, TSet<FName>& OutRemovablePackages)
{
	FMemoryUsageReferenceProcessor ReferenceProcessor;
	ReferenceProcessor.Init(PackagesToUnload);
	PerformReachabilityAnalysis(ReferenceProcessor);
	ReferenceProcessor.GetUnreachablePackages(OutRemovablePackages);
}

void GetUnremovablePackages(const TArray<FName>& PackagesToUnload, TSet<FName>& OutUnremovablePackages)
{
	FMemoryUsageReferenceProcessor ReferenceProcessor;
	ReferenceProcessor.Init(PackagesToUnload);
	PerformReachabilityAnalysis(ReferenceProcessor);
	TSet<FName> UnreachablePackages;
	ReferenceProcessor.GetUnreachablePackages(UnreachablePackages);

	TSet<FName> Dependencies;
	Internal::GetDependenciesCombined(PackagesToUnload, Dependencies);

	for (const auto& Package : Dependencies)
	{
		if (!UnreachablePackages.Contains(Package) && StaticFindObjectFast(UPackage::StaticClass(), nullptr, Package, EFindObjectFlags::ExactClass) != nullptr)
		{
			OutUnremovablePackages.Add(Package);
		}
	}
}

static void GetTransitiveDependenciesAssetRegistry(FName PackageName, TSet<FName>& OutDependencies)
{
	FAssetRegistryModule& AssetRegistryModule = GetAssetRegistryModule();

	TArray<FName> PackageQueue;
	TSet<FName> ExaminedPackages;
	TArray<FName> PackageDependencies;
	OutDependencies.Empty();

	PackageQueue.Push(PackageName);

	while (PackageQueue.Num() > 0)
	{
		const FName& CurrentPackage = PackageQueue.Pop();
		if (ExaminedPackages.Contains(CurrentPackage))
		{
			continue;
		}

		ExaminedPackages.Add(CurrentPackage);

		if (CurrentPackage != PackageName && !OutDependencies.Contains(CurrentPackage))
		{
			OutDependencies.Add(CurrentPackage);
		}

		PackageDependencies.Empty();
		AssetRegistryModule.Get().GetDependencies(CurrentPackage, PackageDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

		for (const auto& Package : PackageDependencies)
		{
			if (!ExaminedPackages.Contains(Package))
			{
				PackageQueue.Push(Package);
			}
		}
	}
}

static void GetTransitiveDependenciesPackageStore(FName PackageName, TSet<FName>& OutDependencies)
{
	const FPackageId PackageId = FPackageId::FromName(PackageName);
	TSet<FPackageId> TransitiveDependencies;
	if (!FPackageDependenciesLazyDatabase::Get().GetDependencies(PackageId, TransitiveDependencies))
	{
		return;
	}

	OutDependencies.Empty();

	for (const FPackageId DependencyId : TransitiveDependencies)
	{
		FName Name;
		if (FPackageStoreLazyDatabase::Get().GetPackageNameFromId(DependencyId, Name))
		{
			OutDependencies.Add(Name);
		}
		else
		{
			OutDependencies.Add(FName(EName::Package, DependencyId.Value()));
		}
	}
}

void GetTransitiveDependencies(FName PackageName, TSet<FName>& OutDependencies)
{
	if (CVarMemQueryUsePackageStore->GetBool() && FPackageStore::Get().HasAnyBackendsMounted())
	{
		GetTransitiveDependenciesPackageStore(PackageName, OutDependencies);
	}
	else
	{
		GetTransitiveDependenciesAssetRegistry(PackageName, OutDependencies);
	}
}

void SortPackagesBySize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TSet<FName>& Packages, TMap<FName, uint64>& OutPackagesWithSize)
{
	GetPackagesSize(MemoryUsageInfoProvider, Packages, OutPackagesWithSize);
	OutPackagesWithSize.ValueSort([&](const uint64& A, const uint64& B) { return A > B; });
}

void GetPackagesSize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TSet<FName>& Packages, TMap<FName, uint64>& OutPackagesWithSize)
{
	for (const auto& Package : Packages)
	{
		uint64 Size = MemoryUsageInfoProvider->GetAssetMemoryUsage(Package);
		OutPackagesWithSize.Add(Package, Size);
	}
}

static void RemoveNonExistentPackagesAssetRegistry(TMap<FName, uint64>& OutPackagesWithSize)
{
	FAssetRegistryModule& AssetRegistryModule = GetAssetRegistryModule();

	for (auto It = OutPackagesWithSize.CreateIterator(); It; ++It)
	{
		if (!AssetRegistryModule.Get().DoesPackageExistOnDisk(It.Key()))
		{
			It.RemoveCurrent();
		}
	}
}

static void RemoveNonExistentPackagesPackageStore(TMap<FName, uint64>& OutPackagesWithSize)
{
	for (auto It = OutPackagesWithSize.CreateIterator(); It; ++It)
	{
		if (!FPackageStoreLazyDatabase::Get().DoesPackageExists(It.Key()))
		{
			It.RemoveCurrent();
		}
	}
}

void RemoveNonExistentPackages(TMap<FName, uint64>& OutPackagesWithSize)
{
	if (CVarMemQueryUsePackageStore->GetBool() && FPackageStore::Get().HasAnyBackendsMounted())
	{
		RemoveNonExistentPackagesPackageStore(OutPackagesWithSize);
	}
	else
	{
		RemoveNonExistentPackagesAssetRegistry(OutPackagesWithSize);
	}
}

void RemoveFilteredPackages(TMap<FName, uint64>& OutPackagesWithSize, const FString& AssetSubstring)
{
	for (auto It = OutPackagesWithSize.CreateIterator(); It; ++It)
	{
		FString KeyString = It.Key().ToString();
		if (!KeyString.Contains(AssetSubstring))
		{
			It.RemoveCurrent();
		}
	}
}

static int32 DefaultResultLimit = 15;

void PrintTagsWithSize(FOutputDevice& Ar, const TMap<FName, uint64>& TagsWithSize, const TCHAR* Name, bool bTruncate /* = false */, int32 Limit /* = -1 */, bool bCSV /* = false */)
{
	uint64 TotalSize = 0U;
	static const FString NoScopeString(TEXT("No scope"));

	if (Limit < 0)
	{
		Limit = DefaultResultLimit;
	}

	int32 Num = TagsWithSize.Num();
	int32 TagsToDisplay = bTruncate ? FMath::Min(Num, Limit) : Num;
	int It = 0;

	if (bCSV)
	{
		Ar.Logf(TEXT(",Name,SizeMB,SizeKB"));
	}

	for (auto& Elem : TagsWithSize)
	{
		if (It++ >= TagsToDisplay)
		{
			break;
		}

		TotalSize += Elem.Value;
		const FString KeyName = Elem.Key.IsValid() ? Elem.Key.ToString() : NoScopeString;

		const float SizeMB = Elem.Value / (1024.0f * 1024.0f);
		const float SizeKB = Elem.Value / 1024.0f;

		if (bCSV)
		{
			Ar.Logf(TEXT(",%s,%.2f,%.2f"), *KeyName, SizeMB, SizeKB);
		}
		else
		{
			Ar.Logf(TEXT("%s - %.2f MB (%.2f KB)"), *KeyName, SizeMB, SizeKB);
		}
	}

	if (TagsToDisplay < Num && !bCSV)
	{
		Ar.Logf(TEXT("----------------------------------------------------------"));
		Ar.Logf(TEXT("<<truncated>> - displayed %d out of %d %s."), TagsToDisplay, Num, Name);
	}

	const float TotalSizeMB = TotalSize / (1024.0f * 1024.0f);
	const float TotalSizeKB = TotalSize / 1024.0f;

	if (bCSV)
	{
		Ar.Logf(TEXT(",TOTAL,%.2f,%.2f"), TotalSizeMB, TotalSizeKB);
	}
	else
	{
		Ar.Logf(TEXT("TOTAL: %.2f MB (%.2f KB)"), TotalSizeMB, TotalSizeKB);
	}
}

} // namespace Internal

} // namespace MemoryUsageQueries
