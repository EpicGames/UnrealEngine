// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryUsageQueries.h"
#include "MemoryUsageQueriesConfig.h"
#include "MemoryUsageQueriesPrivate.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/WildcardString.h"
#include "HAL/FileManager.h"
#include "Templates/Greater.h"

struct FAssetMemoryBreakdown
{
	FAssetMemoryBreakdown()
	{
	}

	uint64 ExclusiveSize = 0;
	uint64 UniqueSize = 0;
	uint64 SharedSize = 0;
	uint64 TotalSize = 0;
};

struct FAssetMemoryDetails
{
	FAssetMemoryDetails()
	{
	}

	// Assets Package Name
	FName PackageName;

	FAssetMemoryBreakdown MemoryBreakdown;

	// List of dependencies for this asset
	TSet<FName> Dependencies;

	TMap<FName, FAssetMemoryBreakdown> DependenciesToMemoryMap;

	int32 UniqueRefCount = 0;
	int32 SharedRefCount = 0;
};

// Helper struct that scopes an output device, and will automatically clean up any log files when leaving scope
struct FScopedOutputDevice
{
	FOutputDevice* CurrentOutputDevice = nullptr;

	FOutputDeviceArchiveWrapper* FileArWrapper = nullptr;
	FArchive* FileAr = nullptr;

	explicit FScopedOutputDevice(FOutputDevice* InDefaultOutputDevice) :
		CurrentOutputDevice(InDefaultOutputDevice)
		
	{
	}

	~FScopedOutputDevice()
	{
		if (FileArWrapper != nullptr)
		{
			FileArWrapper->TearDown();
		}

		delete FileArWrapper;
		delete FileAr;
	}

	FOutputDevice& GetOutputDevice() const
	{
		return *CurrentOutputDevice;
	}

	void OpenLogFile(const FString& LogFileName, bool bCSV)
	{
		if (FileArWrapper != nullptr || LogFileName.IsEmpty())
		{
			// Log file is already open or LogFileName is empty
			return;
		}

#if ALLOW_DEBUG_FILES
		// Create folder for MemQuery files.
		const FString OutputDir = FPaths::ProfilingDir() / TEXT("MemQuery");
		IFileManager::Get().MakeDirectory(*OutputDir, true);

		FString FileTimeString = FString::Printf(TEXT("_%s"), *FDateTime::Now().ToString(TEXT("%H%M%S")));
		const FString FileExtension = (bCSV ? TEXT(".csv") : TEXT(".memquery"));
		const FString LogFilename = OutputDir / (LogFileName + FileTimeString + FileExtension);

		FileAr = IFileManager::Get().CreateDebugFileWriter(*LogFilename);

		if (FileAr != nullptr)
		{
			FileArWrapper = new FOutputDeviceArchiveWrapper(FileAr);
				
			if (FileArWrapper != nullptr)
			{
				CurrentOutputDevice = FileArWrapper;
			}
			else
			{
				delete FileAr;
				FileAr = nullptr;
			}
		}
#endif
	}

	// Delete default constructor to force proper usage
	FScopedOutputDevice() = delete;
};

/** Structure that will parse a string and fill out some commonly used parameters per command */
struct FCommonParameters
{
	int32 Limit = -1;

	bool bTruncate = true;
	bool bCSV = false;
	
	FName Group = NAME_None;
	FName Class = NAME_None;
	FName Category = NAME_None;

	FString Name;
	FString Names;
	FString AssetName;
	FString LogFileName;

	explicit FCommonParameters(const TCHAR* Args)
	{
		bTruncate = !FParse::Param(Args, TEXT("notrunc"));
		bCSV = FParse::Param(Args, TEXT("csv"));

		FParse::Value(Args, TEXT("Name="), Name);
		FParse::Value(Args, TEXT("Names="), Names);
		FParse::Value(Args, TEXT("Limit="), Limit);
		FParse::Value(Args, TEXT("Log="), LogFileName);
		FParse::Value(Args, TEXT("Asset="), AssetName);

		FString GroupName;
		if (FParse::Value(Args, TEXT("Group="), GroupName))
		{
			Group = FName(*GroupName);
		}

		FString ClassName;
		if (FParse::Value(Args, TEXT("Class="), ClassName))
		{
			Class = FName(*ClassName);
		}

		FString CategoryName;
		if (FParse::Value(Args, TEXT("Category="), CategoryName))
		{
			Category = FName(*CategoryName);
		}
	}

	FCommonParameters() = delete;
};

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryUsage(
	TEXT("MemQuery.Usage"),
	TEXT("Name=<AssetName> Prints memory usage of the specified asset."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	FName LongPackageName;
	if (MemoryUsageQueries::Internal::GetLongName(CommonArgs.Name, LongPackageName, &Ar))
	{
		uint64 ExclusiveSize = 0U;
		uint64 InclusiveSize = 0U;

		MemoryUsageQueries::GetMemoryUsage(MemQueryInfoProvider, LongPackageName, ExclusiveSize, InclusiveSize);
		ScopedOutputDevice.GetOutputDevice().Logf(TEXT("MemoryUsage: ExclusiveSize: %.2f MiB (%.2f KiB); InclusiveSize: %.2f MiB (%.2f KiB)"),
			ExclusiveSize / (1024.f * 1024.f),
			ExclusiveSize / 1024.f,
			InclusiveSize / (1024.f * 1024.f),
			InclusiveSize / 1024.f);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryCombinedUsage(
	TEXT("MemQuery.CombinedUsage"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Prints combined memory usage of the specified assets (including all dependencies)."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		uint64 TotalSize;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetMemoryUsageCombined(MemQueryInfoProvider, LongPackageNames.Array(), TotalSize);
		ScopedOutputDevice.GetOutputDevice().Logf(TEXT("MemoryUsageCombined: TotalSize: %.2f MiB (%.2f KiB)"), TotalSize / (1024.f * 1024.f), TotalSize / 1024.f);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQuerySharedUsage(
	TEXT("MemQuery.SharedUsage"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Prints shared memory usage of the specified assets (including only dependencies shared by the specified assets)."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		uint64 SharedSize;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetMemoryUsageShared(MemQueryInfoProvider, LongPackageNames.Array(), SharedSize);
		ScopedOutputDevice.GetOutputDevice().Logf(TEXT("MemoryUsageShared: SharedSize: %.2f MiB (%.2f KiB)"), SharedSize / (1024.f * 1024.f), SharedSize / 1024.f);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryUniqueUsage(
	TEXT("MemQuery.UniqueUsage"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Prints unique memory usage of the specified assets (including only dependencies unique to the specified assets)."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		uint64 UniqueSize = 0U;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetMemoryUsageUnique(MemQueryInfoProvider, LongPackageNames.Array(), UniqueSize);
		ScopedOutputDevice.GetOutputDevice().Logf(TEXT("MemoryUsageUnique: UniqueSize: %.2f MiB (%.2f KiB)"), UniqueSize / (1024.f * 1024.f), UniqueSize / 1024.f);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryCommonUsage(
	TEXT("MemQuery.CommonUsage"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Prints common memory usage of the specified assets (including only dependencies that are not unique to the specified assets)."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		uint64 CommonSize = 0U;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetMemoryUsageCommon(MemQueryInfoProvider, LongPackageNames.Array(), CommonSize);
		ScopedOutputDevice.GetOutputDevice().Logf(TEXT("MemoryUsageCommon: CommonSize: %.2f MiB (%.2f KiB)"), CommonSize / (1024.f * 1024.f), CommonSize / 1024.f);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryDependencies(
	TEXT("MemQuery.Dependencies"),
	TEXT("Name=<AssetName> Limit=<n> Lists dependencies of the specified asset, sorted by size."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	FName LongPackageName;
	if (MemoryUsageQueries::Internal::GetLongName(CommonArgs.Name, LongPackageName, &Ar))
	{
		TMap<FName, uint64> DependenciesWithSize;

		MemoryUsageQueries::GetDependenciesWithSize(MemQueryInfoProvider, LongPackageName, DependenciesWithSize);
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), DependenciesWithSize, TEXT("Dependencies"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryCombinedDependencies(
	TEXT("MemQuery.CombinedDependencies"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Limit=<n> Lists n largest dependencies of the specified assets, sorted by size."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		TMap<FName, uint64> CombinedDependenciesWithSize;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetDependenciesWithSizeCombined(MemQueryInfoProvider, LongPackageNames.Array(), CombinedDependenciesWithSize);
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), CombinedDependenciesWithSize, TEXT("Combined Dependencies"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQuerySharedDependencies(
	TEXT("MemQuery.SharedDependencies"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Limit=<n> Lists n largest dependencies that are shared by the specified assets, sorted by size."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		TMap<FName, uint64> SharedDependenciesWithSize;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetDependenciesWithSizeShared(MemQueryInfoProvider, LongPackageNames.Array(), SharedDependenciesWithSize);
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), SharedDependenciesWithSize, TEXT("Shared Dependencies"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryUniqueDependencies(
	TEXT("MemQuery.UniqueDependencies"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Limit=<n> Lists n largest dependencies that are unique to the specified assets, sorted by size."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		TMap<FName, uint64> UniqueDependenciesWithSize;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetDependenciesWithSizeUnique(MemQueryInfoProvider, LongPackageNames.Array(), UniqueDependenciesWithSize);
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), UniqueDependenciesWithSize, TEXT("Unique Dependencies"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryCommonDependencies(
	TEXT("MemQuery.CommonDependencies"),
	TEXT("Names=\"<AssetName1> <AssetName2> ...\" Limit=<n> Lists n largest dependencies that are NOT unique to the specified assets, sorted by size."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	if (!CommonArgs.Names.IsEmpty())
	{
		TArray<FString> Packages;
		CommonArgs.Names.ParseIntoArrayWS(Packages);
		TMap<FName, uint64> CommonDependenciesWithSize;
		TSet<FName> LongPackageNames;
		if (!MemoryUsageQueries::Internal::GetLongNames(Packages, LongPackageNames, &Ar))
		{
			return;
		}

		MemoryUsageQueries::GetDependenciesWithSizeCommon(MemQueryInfoProvider, LongPackageNames.Array(), CommonDependenciesWithSize);
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), CommonDependenciesWithSize, TEXT("Common Dependencies"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQuerySavings(
	TEXT("MemQuery.Savings"),
	TEXT("Limit=<n> Lists potential savings among %s. How much memory can be saved it we delete certain object."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	const UMemoryUsageQueriesConfig* Config = GetDefault<UMemoryUsageQueriesConfig>();

	// Necessary for FParse::Command
	const TCHAR* Command = *Cmd;

	for (auto It = Config->SavingsPresets.CreateConstIterator(); It; ++It)
	{
		if (!FParse::Command(&Command, *It.Key()))
		{
			continue;
		}

		TMap<FName, uint64> PresetSavings;
		TSet<FName> Packages;

		UClass* SavingsClass = FindObject<UClass>(nullptr, *It.Value());
		if (SavingsClass != nullptr)
		{
			TArray<UClass*> DerivedClasses;
			TArray<UClass*> DerivedResults;

			GetDerivedClasses(SavingsClass, DerivedClasses, true);

			for (UClass* DerivedClass : DerivedClasses)
			{
				UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(DerivedClass);
				if (BPClass != nullptr)
				{
					DerivedResults.Reset();
					GetDerivedClasses(BPClass, DerivedResults, false);
					if (DerivedResults.Num() == 0)
					{
						Packages.Add(DerivedClass->GetPackage()->GetFName());
					}
				}
			}
		}

		for (const auto& Package : Packages)
		{
			uint64 Size = 0;
			MemoryUsageQueries::GetMemoryUsageUnique(MemQueryInfoProvider, TArray{Package}, Size);
			PresetSavings.Add(Package, Size);
		}

		PresetSavings.ValueSort(TGreater<uint64>());
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), PresetSavings, *FString::Printf(TEXT("possible savings")), CommonArgs.bTruncate, CommonArgs.bCSV);
	}
}));

#if ENABLE_LOW_LEVEL_MEM_TRACKER

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryListAssets(
	TEXT("MemQuery.ListAssets"),
	TEXT("Asset=<AssetNameSubstring> Group=<GroupName> Class=<ClassName> Limit=<n> Lists n largest assets."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	bool bSuccess;
	TMap<FName, uint64> AssetsWithSize;

	if (CommonArgs.Group != NAME_None || CommonArgs.Class != NAME_None)
	{
		bSuccess = MemoryUsageQueries::GetFilteredPackagesWithSize(AssetsWithSize, CommonArgs.Group, CommonArgs.AssetName, CommonArgs.Class);
	}
	else
	{
		// TODO - Implement using faster path if there are no group / class filters
		bSuccess = MemoryUsageQueries::GetFilteredPackagesWithSize(AssetsWithSize, CommonArgs.Group, CommonArgs.AssetName, CommonArgs.Class);
	}

	if (bSuccess)
	{
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), AssetsWithSize, TEXT("largest assets"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryListAssetsCategorized(
	TEXT("MemQuery.ListAssetsCategorized"),
	TEXT("Asset=<AssetNameSubstring> Group=<GroupName> Class=<ClassName> Category=<CategoryName(None,Assets,AssetClasses)> Limit=<n> Lists n largest assets categorized by Category."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	bool bSuccess;
	TMap<FName, uint64> AssetsWithSize;
	bSuccess = MemoryUsageQueries::GetFilteredPackagesCategorizedWithSize(AssetsWithSize, CommonArgs.Group, CommonArgs.AssetName, CommonArgs.Class, CommonArgs.Category, &ScopedOutputDevice.GetOutputDevice());

	if (bSuccess)
	{
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), AssetsWithSize, TEXT("largest assets"), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryListClasses(
	TEXT("MemQuery.ListClasses"),
	TEXT("Group=<GroupName> Asset=<AssetName> Limit=<n> Lists n largest classes."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	TMap<FName, uint64> ClassesWithSize;
	if (MemoryUsageQueries::GetFilteredClassesWithSize(ClassesWithSize, CommonArgs.Group, CommonArgs.AssetName, &Ar))
	{
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), ClassesWithSize, *FString::Printf(TEXT("Largest Classes")), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryListGroups(
	TEXT("MemQuery.ListGroups"),
	TEXT("Asset=<AssetName> Class=<ClassName> Limit=<n> Lists n largest groups."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	TMap<FName, uint64> GroupsWithSize;
	if (MemoryUsageQueries::GetFilteredGroupsWithSize(GroupsWithSize, CommonArgs.AssetName, CommonArgs.Class, &Ar))
	{
		MemoryUsageQueries::Internal::PrintTagsWithSize(ScopedOutputDevice.GetOutputDevice(), GroupsWithSize, *FString::Printf(TEXT("Largest Groups")), CommonArgs.bTruncate, CommonArgs.Limit, CommonArgs.bCSV);
	}
}));

// Will return true if the Package Name matches any of the conditions in the array of Paths
static bool PackageNameMatches(const FString& PackageName, const TArray<FString>& Conditions)
{
	for (const FString& Condition : Conditions)
	{
		if ((FWildcardString::ContainsWildcards(*Condition) && FWildcardString::IsMatch(*Condition, *PackageName)) || (PackageName.Contains(Condition)))
		{
			return true;
		}
	}

	return false;
}

FAutoConsoleCommandWithWorldArgsAndOutputDevice GMemQueryCollections(
	TEXT("MemQuery.Collection"),
	TEXT("Lists memory used by a collection. Can show dependency breakdown. Pass -showdeps to list dependencies."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(
		[](const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
{
	const IMemoryUsageInfoProvider* MemQueryInfoProvider = MemoryUsageQueries::GetCurrentMemoryUsageInfoProvider();
	if (!MemQueryInfoProvider)
	{
		Ar.Logf(ELogVerbosity::Error, TEXT("MemoryUsageInfoProvider Error: Provider is not available. Please run with -LLM"));
		return;
	}
	const FString Cmd = FString::Join(Args, TEXT(" "));

	FScopedOutputDevice ScopedOutputDevice(&Ar);
	FCommonParameters CommonArgs(*Cmd);

	ScopedOutputDevice.OpenLogFile(CommonArgs.LogFileName, CommonArgs.bCSV);

	const bool bShowDependencies = FParse::Param(*Cmd, TEXT("ShowDeps"));

	// Necessary for FParse::Command
	const TCHAR* Command = *Cmd;

	FOutputDevice* CurrentOutputDevice = &ScopedOutputDevice.GetOutputDevice();

	const UMemoryUsageQueriesConfig* Config = GetDefault<UMemoryUsageQueriesConfig>();
	for (auto It = Config->Collections.CreateConstIterator(); It; ++It)
	{
		const FCollectionInfo& CollectionInfo = *It;

		if (!FParse::Command(&Command, *CollectionInfo.Name))
		{
			continue;
		}

		// Retrieve a list of all assets that have allocations we are currently tracking.
		TMap<FName, uint64> AssetsWithSize;
		bool bSuccess = MemoryUsageQueries::GetFilteredPackagesWithSize(AssetsWithSize, NAME_None, "", NAME_None);

		if (!bSuccess)
		{
			Ar.Logf(TEXT("Failed to gather assets for Collection %s"), *CollectionInfo.Name);
			break;
		}

		// See if any of the asset paths match those of our matching paths and are valid
		TArray<FName> PackageNames;

		TMap<FName, FAssetMemoryDetails> AssetMemoryMap;
		for (const TPair<FName, uint64>& AssetSizePair : AssetsWithSize)
		{
			const FString& PackageName = AssetSizePair.Key.ToString();

			if (!FPackageName::IsValidLongPackageName(PackageName))
			{
				continue;
			}

			// If path is Included and NOT Excluded, it's a valid asset to consider
			if (PackageNameMatches(PackageName, CollectionInfo.Includes) && !PackageNameMatches(PackageName, CollectionInfo.Excludes))
			{
				FAssetMemoryDetails& AssetMemory = AssetMemoryMap.Add(AssetSizePair.Key);
				AssetMemory.MemoryBreakdown.ExclusiveSize = AssetSizePair.Value;

				FName LongPackageName;
				if (!MemoryUsageQueries::Internal::GetLongName(PackageName, LongPackageName, &Ar))
				{
					Ar.Logf(TEXT("Failed to get long package name for Asset %s"), *PackageName);
				}
				MemoryUsageQueries::Internal::GetTransitiveDependencies(LongPackageName, AssetMemory.Dependencies);
				AssetMemory.Dependencies.Add(LongPackageName);

				PackageNames.Add(LongPackageName);
			}
		}

		// Gather list of dependencies. Internal dependencies are confined only to the set of packages passed in.
		// External are dependencies that have additional references outside the set of packages passed in.
		TMap<FName, uint64> InternalDependencies;
		TMap<FName, uint64> ExternalDependencies;
		MemoryUsageQueries::GatherDependenciesForPackages(MemQueryInfoProvider, PackageNames, InternalDependencies, ExternalDependencies);

		uint64 TotalCollectionSize = 0;

		// Determine which category where each assets dependency should reside
		for (TPair<FName, FAssetMemoryDetails>& Asset : AssetMemoryMap)
		{
			FAssetMemoryDetails& AssetMemory = Asset.Value;
			FAssetMemoryBreakdown& AssetMemoryDetails = AssetMemory.MemoryBreakdown;

			for (FName& Dep : Asset.Value.Dependencies)
			{
				// Don't want to count asset itself, plus some dependencies might refer to other assets in the map
				if (AssetMemoryMap.Contains(Dep))
				{
					continue;
				}

				FAssetMemoryBreakdown DependencyMemory;
				uint64* UniqueMemory = InternalDependencies.Find(Dep);
				uint64* SharedMemory = ExternalDependencies.Find(Dep);
				bool bRecordDependency = false;

				if (UniqueMemory != nullptr && *UniqueMemory != 0)
				{
					DependencyMemory.UniqueSize = *UniqueMemory;
					AssetMemoryDetails.UniqueSize += DependencyMemory.UniqueSize;
					AssetMemory.UniqueRefCount++;
					bRecordDependency = true;
				}

				if (SharedMemory != nullptr && *SharedMemory != 0)
				{
					DependencyMemory.SharedSize = *SharedMemory;
					AssetMemoryDetails.SharedSize += DependencyMemory.SharedSize;
					AssetMemory.SharedRefCount++;
					bRecordDependency = true;
				}

				if (bRecordDependency)
				{
					AssetMemory.DependenciesToMemoryMap.Add(Dep, DependencyMemory);
				}
			}

			AssetMemoryDetails.TotalSize = AssetMemoryDetails.ExclusiveSize + AssetMemoryDetails.UniqueSize;
			TotalCollectionSize += AssetMemoryDetails.TotalSize;
		}

		// Sort by TotalSize
		AssetMemoryMap.ValueSort([](const FAssetMemoryDetails& A, const FAssetMemoryDetails& B) {
			return A.MemoryBreakdown.TotalSize > B.MemoryBreakdown.TotalSize;
			});

		if (CommonArgs.bCSV)
		{
			CurrentOutputDevice->Logf(TEXT(",Asset,Exclusive KiB,Unique Refs KiB,Unique Ref Count,Shared Refs KiB,Shared Ref Count,Total KiB"));
		}
		else
		{
			CurrentOutputDevice->Logf(
				TEXT(" %100s %20s %20s %15s %20s %15s %25s"),
				TEXT("Asset"),
				TEXT("Exclusive KiB"),
				TEXT("Unique Refs KiB"),
				TEXT("Unique Ref Count"),
				TEXT("Shared Refs KiB"),
				TEXT("Shared Ref Count"),
				TEXT("Total KiB")
			);
		}

		// Asset listing
		for (TPair<FName, FAssetMemoryDetails>& Asset : AssetMemoryMap)
		{
			FAssetMemoryDetails& AssetMemory = Asset.Value;
			FAssetMemoryBreakdown& AssetMemoryDetails = AssetMemory.MemoryBreakdown;

			if (CommonArgs.bCSV)
			{
				CurrentOutputDevice->Logf(TEXT(",%s,%.2f,%.2f,%d,%.2f,%d,%.2f"), *Asset.Key.ToString(),
					AssetMemoryDetails.ExclusiveSize / 1024.f,
					AssetMemoryDetails.UniqueSize / 1024.f,
					AssetMemory.UniqueRefCount,
					AssetMemoryDetails.SharedSize / 1024.f,
					AssetMemory.SharedRefCount,
					AssetMemoryDetails.TotalSize / 1024.f);
			}
			else
			{
				CurrentOutputDevice->Logf(
					TEXT(" %100s %20.2f %20.2f %15d %20.2f %15d %25.2f"),
					*Asset.Key.ToString(),
					AssetMemoryDetails.ExclusiveSize / 1024.f,
					AssetMemoryDetails.SharedSize / 1024.f,
					AssetMemory.SharedRefCount,
					AssetMemoryDetails.UniqueSize / 1024.f,
					AssetMemory.UniqueRefCount,
					AssetMemoryDetails.TotalSize / 1024.f
				);
			}
		}

		// Asset dependencies listing
		if (bShowDependencies)
		{
			if (CommonArgs.bCSV)
			{
				CurrentOutputDevice->Logf(TEXT(",Asset,Dependency,Unique KiB,Shared KiB"));
			}
			else
			{
				CurrentOutputDevice->Logf(TEXT(" %100s %100s %20s %20s"),
					TEXT("Asset"),
					TEXT("Dependency"),
					TEXT("Unique KiB"),
					TEXT("Shared KiB")
				);
			}

			for (TPair<FName, FAssetMemoryDetails>& Asset : AssetMemoryMap)
			{
				for (TPair<FName, FAssetMemoryBreakdown>& Dep : Asset.Value.DependenciesToMemoryMap)
				{
					const FString DependencyAssetName = Dep.Key.ToString();
					const FAssetMemoryBreakdown& DepedencyMemoryDetails = Dep.Value;

					if (CommonArgs.bCSV)
					{
						CurrentOutputDevice->Logf(TEXT(",%s,%s,%.2f,%.2f"), *Asset.Key.ToString(), *DependencyAssetName,
							DepedencyMemoryDetails.UniqueSize / 1024.f, DepedencyMemoryDetails.SharedSize / 1024.f);
					}
					else
					{
						CurrentOutputDevice->Logf(TEXT(" %100s %100s %20.2f %20.2f"),
							*Asset.Key.ToString(),
							*DependencyAssetName,
							DepedencyMemoryDetails.UniqueSize / 1024.f,
							DepedencyMemoryDetails.SharedSize / 1024.f
						);
					}
				}
			}
		}

		if (CommonArgs.bCSV)
		{
			CurrentOutputDevice->Logf(TEXT(",TOTAL KiB,%.2f"), TotalCollectionSize / 1024.f);
		}
		else
		{
			CurrentOutputDevice->Logf(TEXT("TOTAL KiB: %.2f"), TotalCollectionSize / 1024.f);
		}
	}
}));

#endif
