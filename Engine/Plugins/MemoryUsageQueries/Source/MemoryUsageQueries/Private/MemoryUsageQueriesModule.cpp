// Copyright Epic Games, Inc. All Rights Reserved.

#include "MemoryUsageQueriesModule.h"
#include "MemoryUsageQueries.h"

#include "Engine/Console.h"
#include "Modules/ModuleManager.h"

void FMemoryUsageQueriesModule::StartupModule()
{
}

void FMemoryUsageQueriesModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FMemoryUsageQueriesModule, MemoryUsageQueries);
