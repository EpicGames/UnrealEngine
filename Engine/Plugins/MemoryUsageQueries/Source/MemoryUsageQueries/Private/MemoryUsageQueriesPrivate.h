// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/FastReferenceCollector.h"

class IMemoryUsageInfoProvider;

namespace MemoryUsageQueries::Internal
{

class FMemoryUsageReferenceProcessor : public FSimpleReferenceProcessorBase
{
public:
	enum EMode
	{
		Full,
		Excluding
	};

private:
	TBitArray<> Excluded;

	TBitArray<> ReachableFull;
	TBitArray<> ReachableExcluded;

	TArray<UObject*> RootSetPackages;

	EMode Mode;

public:
	FMemoryUsageReferenceProcessor();

	void Init(const TArray<FName>& PackageNames);
	TArray<UObject*>& GetRootSet();
	void HandleTokenStreamObjectReference(UE::GC::FWorkerContext& Context, const UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId TokenIndex, const EGCTokenType TokenType, bool bAllowReferenceElimination);
	void GetUnreachablePackages(TSet<FName>& OutUnreachablePackages);

	void SetMode(EMode InMode) { Mode = InMode; }
};

using FMemoryUsageReferenceCollector = UE::GC::TDefaultCollector<FMemoryUsageReferenceProcessor>;

bool GetLongName(FStringView ShortPackageName, FName& OutLongPackageName, FOutputDevice* ErrorOutput = GLog);
bool GetLongNames(const TArray<FString>& PackageNames, TSet<FName>& OutLongPackageNames, FOutputDevice* ErrorOutput = GLog);

void GetDependenciesCombined(const TArray<FName>& PackageNames, TSet<FName>& OutDependencies);
void GetDependenciesShared(const TArray<FName>& PackageNames, TSet<FName>& OutDependencies);

// Get Packages that would be GC'd if PackagesToUnload were unloaded
void GetRemovablePackages(const TArray<FName>& PackagesToUnload, TSet<FName>& OutRemovablePackages);

// Get Packages that would not be GC'd if PackagesToUnload were unloaded
void GetUnremovablePackages(const TArray<FName>& PackagesToUnload, TSet<FName>& OutUnremovablePackages);

void GetTransitiveDependencies(FName PackageName, TSet<FName>& OutDependencies);

void SortPackagesBySize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TSet<FName>& Packages, TMap<FName, uint64>& OutPackagesWithSize);
void GetPackagesSize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TSet<FName>& Packages, TMap<FName, uint64>& OutPackagesWithSize);

void RemoveNonExistentPackages(TMap<FName, uint64>& OutPackagesWithSize);
void RemoveFilteredPackages(TMap<FName, uint64>& OutPackagesWithSize, const FString& AssetSubstring);

void PrintTagsWithSize(FOutputDevice& Ar, const TMap<FName, uint64>& TagsWithSize, const TCHAR* Name, bool bTruncate = false, int32 Limit = -1, bool bCSV = false);

} // namespace MemoryUsageQueries::Internal
