// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreGlobals.h"

#define UE_API MEMORYUSAGEQUERIES_API


class IMemoryUsageInfoProvider
{
public:
	UE_DEPRECATED(5.6, "Use version that do not accept FOutputDevice, check IsProviderAvailable() and do the logging yourself.")
	UE_API virtual uint64 GetAssetMemoryUsage(FName Asset, FOutputDevice* ErrorOutput) const;
	UE_DEPRECATED(5.6, "Use version that do not accept FOutputDevice, check IsProviderAvailable() and do the logging yourself.")
	UE_API virtual uint64 GetAssetsMemoryUsage(const TSet<FName>& Assets, FOutputDevice* ErrorOutput) const;
	UE_DEPRECATED(5.6, "Use version that do not accept FOutputDevice, check IsProviderAvailable() and do the logging yourself.")
	UE_API virtual void GetAllAssetsWithSize(TMap<FName, uint64>& OutAssets, FOutputDevice* ErrorOutput) const;
	UE_DEPRECATED(5.6, "Use version that do not accept FOutputDevice, check IsProviderAvailable() and do the logging yourself.")
	UE_API virtual uint64 GetAssetsMemoryUsageWithSize(const TSet<FName>& Assets, TMap<FName, uint64>& OutSizes, FOutputDevice* ErrorOutput) const;

	virtual bool IsProviderAvailable() const = 0;
	virtual uint64 GetAssetMemoryUsage(FName Asset) const = 0;
	virtual uint64 GetAssetsMemoryUsage(const TSet<FName>& Assets) const = 0;
	virtual void GetAllAssetsWithSize(TMap<FName, uint64>& OutAssets) const = 0;
	virtual uint64 GetAssetsMemoryUsageWithSize(const TSet<FName>& Assets, TMap<FName, uint64>& OutSizes) const = 0;
};

#undef UE_API
