// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MemoryUsageInfoProvider.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/OutputDeviceRedirector.h"

struct FAutoCompleteCommand;

class FOutputDevice;

namespace MemoryUsageQueries
{
	enum EDependencyType
{
	EDep_All,
	EDep_Removable,
	EDep_NonRemovable
};
MEMORYUSAGEQUERIES_API IMemoryUsageInfoProvider* GetCurrentMemoryUsageInfoProvider();
UE_DEPRECATED(5.6, "Use FName version instead, resolve user input using GetLongName if needed")
MEMORYUSAGEQUERIES_API bool GetMemoryUsage(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FString& PackageName, uint64& OutExclusiveSize, uint64& OutInclusiveSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetMemoryUsageCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutTotalSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetMemoryUsageShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutTotalSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetMemoryUsageUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutUniqueSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetMemoryUsageCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, uint64& OutCommonSize, FOutputDevice* ErrorOutput = GLog);

UE_DEPRECATED(5.6, "Use FName version instead, resolve user input using GetLongName if needed")
MEMORYUSAGEQUERIES_API bool GetDependenciesWithSize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FString& PackageName, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetDependenciesWithSizeCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetDependenciesWithSizeShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetDependenciesWithSizeUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GetDependenciesWithSizeCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize, FOutputDevice* ErrorOutput = GLog);

MEMORYUSAGEQUERIES_API void GetMemoryUsage(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FName& PackageName, uint64& OutExclusiveSize, uint64& OutInclusiveSize);
MEMORYUSAGEQUERIES_API void GetMemoryUsageCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutTotalSize);
MEMORYUSAGEQUERIES_API void GetMemoryUsageShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutTotalSize);
MEMORYUSAGEQUERIES_API void GetMemoryUsageUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutUniqueSize);
MEMORYUSAGEQUERIES_API void GetMemoryUsageCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, uint64& OutCommonSize);

MEMORYUSAGEQUERIES_API void GetDependenciesWithSize(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const FName& PackageName, TMap<FName, uint64>& OutDependenciesWithSize);
MEMORYUSAGEQUERIES_API void GetDependenciesWithSizeCombined(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize);
MEMORYUSAGEQUERIES_API void GetDependenciesWithSizeShared(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize);
MEMORYUSAGEQUERIES_API void GetDependenciesWithSizeUnique(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize);
MEMORYUSAGEQUERIES_API void GetDependenciesWithSizeCommon(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutDependenciesWithSize);

#if ENABLE_LOW_LEVEL_MEM_TRACKER

MEMORYUSAGEQUERIES_API bool GetFilteredPackagesWithSize(TMap<FName, uint64>& OutPackagesWithSize, FName GroupName = NAME_None, FString AssetSubstring = FString(), FName ClassName = NAME_None, FOutputDevice* ErrorOutput = GLog);
MEMORYUSAGEQUERIES_API bool GetFilteredPackagesCategorizedWithSize(TMap<FName, uint64>& OutPackagesWithSize, FName GroupName = NAME_None, FString AssetSubstring = FString(), FName ClassName = NAME_None, FName CategoryName = NAME_None, FOutputDevice* ErrorOutput = GLog);
MEMORYUSAGEQUERIES_API bool GetFilteredClassesWithSize(TMap<FName, uint64>& OutClassesWithSize, FName GroupName = NAME_None, FString AssetName = FString(), FOutputDevice* ErrorOutput = GLog);
MEMORYUSAGEQUERIES_API bool GetFilteredGroupsWithSize(TMap<FName, uint64>& OutGroupsWithSize, FString AssetName = FString(), FName ClassName = NAME_None, FOutputDevice* ErrorOutput = GLog);
UE_DEPRECATED(5.6, "Use TArray<FName> version instead, resolve user input using GetLongNames if needed")
MEMORYUSAGEQUERIES_API bool GatherDependenciesForPackages(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FString>& PackageNames, TMap<FName, uint64>& OutInternalDeps, TMap<FName, uint64>& OutExternalDeps, MemoryUsageQueries::EDependencyType DependencyType = MemoryUsageQueries::EDependencyType::EDep_All, FOutputDevice* ErrorOutput = GLog);
MEMORYUSAGEQUERIES_API void GatherDependenciesForPackages(const IMemoryUsageInfoProvider* MemoryUsageInfoProvider, const TArray<FName>& PackageNames, TMap<FName, uint64>& OutInternalDeps, TMap<FName, uint64>& OutExternalDeps, MemoryUsageQueries::EDependencyType DependencyType = MemoryUsageQueries::EDependencyType::EDep_All);
#endif

}
