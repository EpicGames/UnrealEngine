// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MemoryUsageQueriesConfig.generated.h"

// “MemQuery Collection” will list memory consumption based on assets discovered that have a path
// matching the one described in the ini file.
// Matching supports Wildcards. e.g. */Game*/*
// This will match assets such as /SomePath/Game/SomeLevelAsset
// 
// Define these in DefaultMemoryUsageQueries.ini in your project – you have to provide preset name and UClass.
// 
// e.g. Include all Characters and Pawns but exclude anything with Item in the path. Demonstrates wildcard usage.
// [/Script/MemoryUsageQueries.MemoryUsageQueriesConfig]
// +Collections = (Name="Pawns", Includes=("*/Character*/*", "/Pawn/"), Excludes=("*/Item*/*"))
//
// A budget can be defined which will compare the total memory against the budget
// e.g.
// [/Script/MemoryUsageQueries.MemoryUsageQueriesConfig]
// +Collections = (Name="Pawns", Includes=("/Character/", "/Pawn/"), BudgetMB=2.0)
//
// "MemQuery Savings" command will list all derived classes with potential savings for the specified preset.
// 
// e.g.
// [/Script/MemoryUsageQueries.MemoryUsageQueriesConfig]
// +SavingsPresets = (("Pawns", "/Script/OurProject.OurCharacter"))
//


USTRUCT()
struct FCollectionInfo
{
	GENERATED_BODY()

	FCollectionInfo() :
		BudgetMB(0.0f)
	{
	}

	// Name of the collection
	UPROPERTY()
	FString Name;

	// Collection of substrings to include whne matching against asset package paths
	UPROPERTY()
	TArray<FString> Includes;

	// Paths of asset package paths to exclude from the results
	UPROPERTY()
	TArray<FString> Excludes;

	// Budget in MB
	UPROPERTY()
	float BudgetMB;
};

UCLASS(config = MemoryUsageQueries)
class UMemoryUsageQueriesConfig : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(config)
	TArray<FCollectionInfo> Collections;

	UPROPERTY(config)
	TMap<FString, FString> SavingsPresets;
};